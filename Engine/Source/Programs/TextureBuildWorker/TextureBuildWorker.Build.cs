// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;
using EpicGames.Core;
using UnrealBuildTool;

// Abstract base class for texture worker targets.  Not a valid target by itself, hence it is not put into a *.target.cs file.
[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public abstract class TextureBuildWorkerTarget : DerivedDataBuildWorkerTarget
{
	public TextureBuildWorkerTarget(TargetInfo Target) : base(Target)
	{
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;

		SolutionDirectory += "/Texture";
	}

}

public class TextureBuildWorker : ModuleRules
{
	public TextureBuildWorker(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.Add("DerivedDataCache");
		PrivateDependencyModuleNames.Add("DerivedDataBuildWorker");
	}
}
